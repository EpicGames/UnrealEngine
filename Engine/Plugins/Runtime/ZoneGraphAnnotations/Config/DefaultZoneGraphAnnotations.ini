[CoreRedirects]
+PackageRedirects=(OldName="/Script/ZoneGraphBehaviors", NewName="/Script/ZoneGraphAnnotations")
+StructRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphBehaviorEventBase", NewName="/Script/ZoneGraphAnnotations.ZoneGraphAnnotationEventBase")
+ClassRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphBehaviorTest", NewName="/Script/ZoneGraphAnnotations.ZoneGraphAnnotationTest")
+StructRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphDangerArea", NewName="/Script/ZoneGraphAnnotations.ZoneGraphDisturbanceArea")
+ClassRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphBehaviorTestingComponent", NewName="/Script/ZoneGraphAnnotations.ZoneGraphAnnotationTestingComponent")
+ClassRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphBehaviorTestingActor", NewName="/Script/ZoneGraphAnnotations.ZoneGraphAnnotationTestingActor")
+ClassRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphAnnotationSubsystem", NewName="/Script/ZoneGraphAnnotations.ZoneGraphAnnotationSubsystem")
+ClassRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphFleeBehavior", NewName="/Script/ZoneGraphAnnotations.ZoneGraphDisturbanceAnnotation")
+ClassRedirects=(OldName="/Script/ZoneGraphBehaviors.ZoneGraphFleeBehaviorBPLibrary", NewName="/Script/ZoneGraphAnnotations.ZoneGraphDisturbanceAnnotationBPLibrary")
+PropertyRedirects=(OldName="ZoneGraphDisturbanceAnnotation.AnnotationTag",NewName="ZoneGraphDisturbanceAnnotation.DangerAnnotationTag")