// Copyright Epic Games, Inc. All Rights Reserved.

#include "Annotations/ZoneGraphDisturbanceAnnotation.h"
#include "Engine/Engine.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphRenderingUtilities.h"
#include "ZoneGraphQuery.h"

#if UE_ENABLE_DEBUG_DRAWING
#include "Engine/Canvas.h"
#endif // UE_ENABLE_DEBUG_DRAWING

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphDisturbanceAnnotation)

#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 6385) // Disable dubious Static Analysis warning C6385 on windows. See MS TechNote Here https://developercommunity2.visualstudio.com/t/C6385-False-Positive/878703
#endif

namespace UE::ZoneGraphAnnotations
{
	
	static float CalculateCost(const float CurrentCost, const float Distance, const float DangerCost)
	{
		// Let zero propagate at the end of the lanes.
		if (FMath::IsNearlyZero(CurrentCost) && FMath::IsNearlyZero(DangerCost))
		{
			return 0.0f;
		}
		return CurrentCost + Distance * (1.0f + DangerCost);
	}
	
} // UE::ZoneGraphAnnotations

//////////////////////////////////////////////////////////////////////////
// UZoneGraphDisturbanceAnnotation
UZoneGraphDisturbanceAnnotation::UZoneGraphDisturbanceAnnotation(const FObjectInitializer& ObjectInitializer)
	: UZoneGraphAnnotationComponent(ObjectInitializer)
{
}

void UZoneGraphDisturbanceAnnotation::PostSubsystemsInitialized()
{
	Super::PostSubsystemsInitialized();

	ZoneGraphSubsystem = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());
	checkf(ZoneGraphSubsystem, TEXT("Expecting ZoneGraphSubsystem to be present."));
}

void UZoneGraphDisturbanceAnnotation::TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& AnnotationTagContainer)
{
	// Decay dangers
	for (int32 Index = 0; Index < Dangers.Num(); Index++)
	{
		FZoneGraphDisturbanceArea& Danger = Dangers[Index];
		Danger.Duration -= DeltaTime;
		if (Danger.Duration < 0.0f)
		{
			Dangers.RemoveAtSwap(Index);
			Index--;
			bDisturbancesChanged = true;
		}
	}

	// Update the affected lanes when needed. 
	if (bDisturbancesChanged)
	{
		UpdateDangerLanes();
		bDisturbancesChanged = false;
		
		UpdateAnnotationTags(AnnotationTagContainer);
	}

#if UE_ENABLE_DEBUG_DRAWING
	if (bEnableDebugDrawing)
	{
		FVector ViewLocation = FVector::ZeroVector;
		FRotator ViewRotation = FRotator::ZeroRotator;
		GetFirstViewPoint(ViewLocation, ViewRotation);

		// The debug draw caches an area around the current view. Update the debug drawing if the view moves far enough from the last snapshot.
		const float UpdateThreshold = GetMaxDebugDrawDistance() * 0.25f;
		if (FVector::DistSquared(LastDebugDrawLocation, ViewLocation) > FMath::Square(UpdateThreshold))
		{
			MarkRenderStateDirty();
		}
	}
#endif // UE_ENABLE_DEBUG_DRAWING

}

void UZoneGraphDisturbanceAnnotation::UpdateDangerLanes()
{
	checkf(ZoneGraphSubsystem, TEXT("Expecting ZoneGraphSubsystem to be present."));

	for (FZoneGraphDataEscapeGraph& EscapeGraph : EscapeGraphs)
	{
		if (EscapeGraph.bInUse)
		{
			EscapeGraph.LanesToEscapeLookup.Reset();
			EscapeGraph.LanesToEscape.Reset();
		}
	}

	// Find which lanes the disturbances affect, and sort them based on which ZoneGraphData they belong to.
	auto FindOverlappingLambda = [this](const FVector& Position, const float Radius, const FZoneGraphTag Tag)
	{
		const FBox QueryBounds = FBox::BuildAABB(Position, FVector(Radius));

		TArray<FZoneGraphLaneHandle> OverlappingLanes;
		if (ZoneGraphSubsystem->FindOverlappingLanes(QueryBounds, AffectedLaneTags, OverlappingLanes))
		{
			for (const FZoneGraphLaneHandle& LaneHandle : OverlappingLanes)
			{
				if (!EscapeGraphs.IsValidIndex(LaneHandle.DataHandle.Index))
				{
					continue;
				}
				FZoneGraphDataEscapeGraph& EscapeGraph = EscapeGraphs[LaneHandle.DataHandle.Index];
				check(EscapeGraph.bInUse);
				
				// Add new lane if not added already.
				if (const int32* EscapeLaneIndex = EscapeGraph.LanesToEscapeLookup.Find(LaneHandle.Index))
				{
					// Update existing
					EscapeGraph.LanesToEscape[*EscapeLaneIndex].Tags.Add(Tag);
				}
				else
				{
					// Add new
					const int32 Index = EscapeGraph.LanesToEscape.Emplace(LaneHandle.Index);
					EscapeGraph.LanesToEscapeLookup.Add(LaneHandle.Index, Index);
					EscapeGraph.LanesToEscape[Index].Tags.Add(Tag);
				}
			}
		}
	};
	
	for (const FZoneGraphDisturbanceArea& Danger : Dangers)
	{
		FindOverlappingLambda(Danger.Position, Danger.Radius, DangerAnnotationTag);
	}

	for (const FZoneGraphObstacleDisturbanceArea& Obstacle : Obstacles)
	{
		FindOverlappingLambda(Obstacle.Position, Obstacle.Radius, ObstacleAnnotationTag);
	}

	for (FZoneGraphDataEscapeGraph& EscapeGraph : EscapeGraphs)
	{
		if (EscapeGraph.bInUse)
		{
			CalculateEscapeGraph(EscapeGraph);
		}
	}
	
	MarkRenderStateDirty();
}

void UZoneGraphDisturbanceAnnotation::CalculateEscapeGraph(FZoneGraphDataEscapeGraph& EscapeGraph)
{
	checkf(ZoneGraphSubsystem, TEXT("Expecting ZoneGraphSubsystem to be present."));
	const FZoneGraphStorage* ZoneStorage = ZoneGraphSubsystem->GetZoneGraphStorage(EscapeGraph.DataHandle);
	if (!ZoneStorage)
	{
		UE_LOGF(LogZoneGraphAnnotations, Error, "%ls: %ls Could not find ZoneStorage.", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(this));
		return;
	}

	struct FEscapeNode
	{
		FEscapeNode() = default;
		FEscapeNode(const int32 InLaneIndex, const float InEscapeCost, const uint8 InSpanIndex, const int32 InExitLaneIndex, const EZoneLaneLinkType InExitLinkType)
			: EscapeLaneIndex(InLaneIndex), ExitLaneIndex(InExitLaneIndex), EscapeCost(InEscapeCost), SpanIndex(InSpanIndex), ExitLinkType(InExitLinkType)
		{}

		bool operator<(const FEscapeNode& Other) const { return EscapeCost < Other.EscapeCost; }

		// Lane index in FZoneGraphDataEscapeGraph::LanesToEscape
		int32 EscapeLaneIndex = 0;
		// Exit lane index in ZoneGraph data.
		int32 ExitLaneIndex = INDEX_NONE;
		// Cumulative cost of reaching this node.
		float EscapeCost = 0.0f;
		// Disturbance lane span index (start/end) 
		uint8 SpanIndex = 0;
		EZoneLaneLinkType ExitLinkType = EZoneLaneLinkType::None;
	};
	
	TArray<FEscapeNode> DisturbanceHeap;
	TArray<FZoneGraphLinkedLane> LinkedLanes;


	auto AddSearchStart = [this, ZoneStorage, &LinkedLanes, &EscapeGraph, &DisturbanceHeap](const int32 EscapeLaneIndex)
	{
		FZoneGraphEscapeLaneAction& EscapeLane = EscapeGraph.LanesToEscape[EscapeLaneIndex];
		const FZoneGraphLaneHandle LaneHandle(EscapeLane.LaneIndex, EscapeGraph.DataHandle);

		// Check if the lane leads out of the danger zone, and add as a starting point for the search.
		for (uint8 SpanIndex = 0; SpanIndex < EscapeLane.SpanCount; SpanIndex += (EscapeLane.SpanCount - 1))
		{
			FZoneGraphEscapeLaneSpan& Span = EscapeLane.Spans[SpanIndex];

			const EZoneLaneLinkType ExitLinkType = SpanIndex == 0 ? EZoneLaneLinkType::Incoming : EZoneLaneLinkType::Outgoing;
			UE::ZoneGraph::Query::GetLinkedLanes(*ZoneStorage, LaneHandle, ExitLinkType, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, LinkedLanes);

			for (const FZoneGraphLinkedLane& LinkedLane : LinkedLanes)
			{
				// If the linked lane is not part of the lanes to escape and passes the escape lanes filter, add this avoided lane as start for the search.
				if (!EscapeGraph.LanesToEscapeLookup.Contains(LinkedLane.DestLane.Index))
				{
					const FZoneLaneData& DestLane = ZoneStorage->Lanes[LinkedLane.DestLane.Index];
					if (EscapeLaneTags.Pass(DestLane.Tags))
					{
						DisturbanceHeap.Emplace(EscapeLaneIndex, /*EscapeCost*/0.0f, SpanIndex, LinkedLane.DestLane.Index, ExitLinkType);
						Span.bLeadsToExit = true;
						Span.ExitLinkType = ExitLinkType;
						Span.EscapeCost = 0.0f;
						Span.ExitLaneIndex = LinkedLane.DestLane.Index;
						Span.bReverseLaneDirection = ExitLinkType == EZoneLaneLinkType::Incoming;
						break;
					}
				}
			}
		}				
	};

	
	const float InvIdealSpanLength = 1.0f / FMath::Max(1.0f, IdealSpanLength);
	float LowestLaneDanger = MAX_flt;

	for (int32 EscapeLaneIndex = 0; EscapeLaneIndex < EscapeGraph.LanesToEscape.Num(); EscapeLaneIndex++)
	{
		FZoneGraphEscapeLaneAction& EscapeLane = EscapeGraph.LanesToEscape[EscapeLaneIndex];
		const FZoneGraphLaneHandle LaneHandle(EscapeLane.LaneIndex, EscapeGraph.DataHandle);
		const FZoneLaneData& Lane = ZoneStorage->Lanes[EscapeLane.LaneIndex];
		
		float LaneLength = 0.0f;
		UE::ZoneGraph::Query::GetLaneLength(*ZoneStorage, LaneHandle, LaneLength);

		// Split the lane up to FZoneGraphEscapeLaneAction::MaxSpans to have more fine grained Disturbance Annotation.
		EscapeLane.SpanCount = FMath::Clamp((uint8)FMath::CeilToInt(LaneLength * InvIdealSpanLength), (uint8)2, FZoneGraphEscapeLaneAction::MaxSpans);
		
		FVector PrevPosition = ZoneStorage->LanePoints[Lane.PointsBegin];
		float PrevDistance = 0.0f;

		float AccumulatedDanger = 0.0f;
		
		for (uint8 SpanIndex = 0; SpanIndex < EscapeLane.SpanCount; SpanIndex++)
		{
			FZoneGraphEscapeLaneSpan& Span = EscapeLane.Spans[SpanIndex];
			
			const float SpanEndFraction = (float)(SpanIndex + 1) / (float)EscapeLane.SpanCount;
			const float Distance = LaneLength * SpanEndFraction;

			// Location and the end of the span, used for evaluating the danger value.
			FZoneGraphLaneLocation LaneLocation;
			UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneStorage, EscapeLane.LaneIndex, Distance, LaneLocation);

			// Location and the middle of the span, used as representative point of the span.
			FZoneGraphLaneLocation MidLaneLocation;
			UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneStorage, EscapeLane.LaneIndex, (PrevDistance + Distance) * 0.5f, MidLaneLocation);

			Span.Position = MidLaneLocation.Position;
			Span.Direction = MidLaneLocation.Tangent;
			Span.SplitDistance = Distance;
			Span.EscapeCost = MAX_flt;
			Span.bReverseLaneDirection = false;
			Span.bLeadsToExit = false;

			// Calculate max danger.
			Span.Danger = 0.0f;
			for (const FZoneGraphDisturbanceArea& Danger : Dangers)
			{
				const FVector ClosestPoint = FMath::ClosestPointOnSegment(Danger.Position, PrevPosition, LaneLocation.Position);
				const float DangerCost = FloatCastChecked<float>(1. - FMath::Min(FVector::Dist2D(Danger.Position, ClosestPoint) / Danger.Radius, 1.), UE::LWC::DefaultFloatPrecision);
				Span.Danger = FMath::Max(Span.Danger, DangerCost);
			}

			for (const FZoneGraphObstacleDisturbanceArea& Obstacle : Obstacles)
			{
				const FVector ClosestPoint = FMath::ClosestPointOnSegment(Obstacle.Position, PrevPosition, LaneLocation.Position);
				const float DangerCost = FloatCastChecked<float>(1. - FMath::Min(FVector::Dist2D(Obstacle.Position, ClosestPoint) / Obstacle.Radius, 1.), UE::LWC::DefaultFloatPrecision);
				Span.Danger = FMath::Max(Span.Danger, DangerCost);
			}

			AccumulatedDanger += Span.Danger;
			
			PrevPosition = LaneLocation.Position;
			PrevDistance = Distance;
		}

		LowestLaneDanger = FMath::Min(LowestLaneDanger, AccumulatedDanger);
		
		EscapeLane.LaneLength = LaneLength;

		AddSearchStart(EscapeLaneIndex);
	}

	// If none of the lanes lead to safety, remove lowest danger lanes and try to connect again.
	// This can happen for example when danger area covers a city block and crosswalks are not allowed as escape lanes.
	if (DisturbanceHeap.IsEmpty() && EscapeGraph.LanesToEscape.Num() > 0)
	{
		// Remove all lanes that have accumulated danger value below the lowest accumulated danger found earlier. 
		const float ExitDangerThreshold = LowestLaneDanger + KINDA_SMALL_NUMBER;
		for (int32 EscapeLaneIndex = 0; EscapeLaneIndex < EscapeGraph.LanesToEscape.Num(); EscapeLaneIndex++)
		{
			FZoneGraphEscapeLaneAction& EscapeLane = EscapeGraph.LanesToEscape[EscapeLaneIndex];
		
			float AccumulatedDanger = 0.0f;
			for (uint8 SpanIndex = 0; SpanIndex < EscapeLane.SpanCount; SpanIndex++)
			{
				FZoneGraphEscapeLaneSpan& Span = EscapeLane.Spans[SpanIndex];
				AccumulatedDanger += Span.Danger;
			}

			if (AccumulatedDanger < ExitDangerThreshold)
			{
				EscapeGraph.LanesToEscape.RemoveAt(EscapeLaneIndex);
				EscapeLaneIndex--;
			}
		}

		// Rebuild LanesToEscapeLookup
		EscapeGraph.LanesToEscapeLookup.Reset();
		for (int32 EscapeLaneIndex = 0; EscapeLaneIndex < EscapeGraph.LanesToEscape.Num(); EscapeLaneIndex++)
		{
			const FZoneGraphEscapeLaneAction& EscapeLane = EscapeGraph.LanesToEscape[EscapeLaneIndex];
			EscapeGraph.LanesToEscapeLookup.Add(EscapeLane.LaneIndex, EscapeLaneIndex);
		}

		// Try again to add the search start locations.
		for (int32 EscapeLaneIndex = 0; EscapeLaneIndex < EscapeGraph.LanesToEscape.Num(); EscapeLaneIndex++)
		{
			AddSearchStart(EscapeLaneIndex);
		}
	}
	
	// Flood exit direction along the danger lanes.
	// 
	// NOTE:A lot of this code feels reversed because we're advancing from the fringe towards the center,
	// but creating data to move opposite direction than the search (i.e. escape towards the fringe).

	static constexpr float DangerPenalty = 10.0f;	// Additional penalty to prefer lanes with less danger.
	static constexpr float AdjacentPenalty = 2.0f;	// Additional penalty for choosing adjacent lane.

	DisturbanceHeap.Heapify();

	while (DisturbanceHeap.Num() > 0)
	{
		FEscapeNode Node = DisturbanceHeap.HeapTop();
		DisturbanceHeap.HeapPopDiscard();

		FZoneGraphEscapeLaneAction& EscapeLane = EscapeGraph.LanesToEscape[Node.EscapeLaneIndex];
		FZoneGraphEscapeLaneSpan& Span = EscapeLane.Spans[Node.SpanIndex];
		const FZoneLaneData& CurrentLane = ZoneStorage->Lanes[EscapeLane.LaneIndex];
		const bool bIsCurrentEscapeLane = EscapeLaneTags.Pass(CurrentLane.Tags);

		bool bAdvanceToNextLane = false;

		if (Node.ExitLinkType == EZoneLaneLinkType::Incoming || Node.ExitLinkType == EZoneLaneLinkType::Outgoing)
		{
			const uint8 LastSpan = Node.ExitLinkType == EZoneLaneLinkType::Incoming ? (EscapeLane.SpanCount - 1) : 0;
			
			if (Node.SpanIndex != LastSpan)
			{
				// Move to next span, if not reached the last span yet.
				const uint8 NextSpanIndex = Node.ExitLinkType == EZoneLaneLinkType::Incoming ? (Node.SpanIndex + 1) : (Node.SpanIndex - 1);
				FZoneGraphEscapeLaneSpan& NextSpan = EscapeLane.Spans[NextSpanIndex];
				const float Danger = (Span.Danger + NextSpan.Danger) * 0.5f;
				const float Distance = FloatCastChecked<float>(FVector::Distance(Span.Position, NextSpan.Position), UE::LWC::DefaultFloatPrecision);
				const float NextCost = UE::ZoneGraphAnnotations::CalculateCost(Node.EscapeCost, Distance, Danger * DangerPenalty);
				if (NextCost < NextSpan.EscapeCost)
				{
					NextSpan.EscapeCost = NextCost;
					NextSpan.bReverseLaneDirection = Node.ExitLinkType == EZoneLaneLinkType::Incoming;
					NextSpan.ExitLinkType = Node.ExitLinkType;
					NextSpan.ExitLaneIndex = Node.ExitLaneIndex;
					DisturbanceHeap.HeapPush(FEscapeNode(Node.EscapeLaneIndex, NextCost, NextSpanIndex, Node.ExitLaneIndex, Node.ExitLinkType));
				}
			}
			else
			{
				// On last span, advance to next lane.
				bAdvanceToNextLane = true;
			}
		}
		else if (Node.ExitLinkType == EZoneLaneLinkType::Adjacent)
		{
			// Advance to both directions when coming via adjacent link.
			if (Node.SpanIndex > 0)
			{
				const uint8 NextSpanIndex = Node.SpanIndex - 1;
				FZoneGraphEscapeLaneSpan& NextSpan = EscapeLane.Spans[NextSpanIndex];
				const float Danger = (Span.Danger + NextSpan.Danger) * 0.5f;
				const float Distance = FloatCastChecked<float>(FVector::Distance(Span.Position, NextSpan.Position), UE::LWC::DefaultFloatPrecision);
				const float NextCost = UE::ZoneGraphAnnotations::CalculateCost(Node.EscapeCost, Distance, Danger * DangerPenalty);
				if (NextCost < NextSpan.EscapeCost)
				{
					NextSpan.EscapeCost = NextCost;
					NextSpan.bReverseLaneDirection = true;
					NextSpan.ExitLinkType = Node.ExitLinkType;
					NextSpan.ExitLaneIndex = Node.ExitLaneIndex;
					DisturbanceHeap.HeapPush(FEscapeNode(Node.EscapeLaneIndex, NextCost, NextSpanIndex, Node.ExitLaneIndex, Node.ExitLinkType));
				}
			}
			
			if (Node.SpanIndex < (EscapeLane.SpanCount - 1))
			{
				const uint8 NextSpanIndex = Node.SpanIndex + 1;
				FZoneGraphEscapeLaneSpan& NextSpan = EscapeLane.Spans[NextSpanIndex];
				const float Danger = (Span.Danger + NextSpan.Danger) * 0.5f;
				const float Distance = FloatCastChecked<float>(FVector::Distance(Span.Position, NextSpan.Position), UE::LWC::DefaultFloatPrecision);
				const float NextCost = UE::ZoneGraphAnnotations::CalculateCost(Node.EscapeCost, Distance, Danger * DangerPenalty);
				if (NextCost < NextSpan.EscapeCost)
				{
					NextSpan.EscapeCost = NextCost;
					NextSpan.bReverseLaneDirection = false;
					NextSpan.ExitLinkType = Node.ExitLinkType;
					NextSpan.ExitLaneIndex = Node.ExitLaneIndex;
					DisturbanceHeap.HeapPush(FEscapeNode(Node.EscapeLaneIndex, NextCost, NextSpanIndex, Node.ExitLaneIndex, Node.ExitLinkType));
				}
			}

			if (Node.SpanIndex == 0 || Node.SpanIndex == (EscapeLane.SpanCount - 1))
			{
				// On last span on either end, advance to next lane.
				bAdvanceToNextLane = true;
			}
		}

		if (bAdvanceToNextLane)
		{
			// Move to next lane
			EZoneLaneLinkType ExitLinkType = Node.ExitLinkType;
			if (ExitLinkType == EZoneLaneLinkType::Adjacent)
			{
				ExitLinkType = Node.SpanIndex == 0 ? EZoneLaneLinkType::Incoming : EZoneLaneLinkType::Outgoing;
			}
			const EZoneLaneLinkType QueryLinkType = ExitLinkType == EZoneLaneLinkType::Incoming ? EZoneLaneLinkType::Outgoing : EZoneLaneLinkType::Incoming;
			
			UE::ZoneGraph::Query::GetLinkedLanes(*ZoneStorage, EscapeLane.LaneIndex, QueryLinkType, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, LinkedLanes);

			for (const FZoneGraphLinkedLane& LinkedLane : LinkedLanes)
			{
				const int32 DestLaneIndex = LinkedLane.DestLane.Index;
				// Dest lane must be part of the lanes to avoid.
				if (const int32* DestEscapeLaneIndex = EscapeGraph.LanesToEscapeLookup.Find(DestLaneIndex))
				{
					const FZoneLaneData& DestLane = ZoneStorage->Lanes[DestLaneIndex];
					const bool bIsDestEscapeLane = EscapeLaneTags.Pass(DestLane.Tags);

					// Allow Disturbance lanes to bleed into non-Disturbance lanes, and non-Disturbance lanes to non-Disturbance lanes too.
					// This allows to "drain" non-Disturbance lanes, but prevents Disturbanceing via non-Disturbance lanes.
					if (bIsCurrentEscapeLane || !bIsDestEscapeLane)
					{
						// Dest lane must be cheaper to reach via this lane.
						FZoneGraphEscapeLaneAction& DestEscapeLane = EscapeGraph.LanesToEscape[*DestEscapeLaneIndex];
						const uint8 NextSpanIndex = ExitLinkType == EZoneLaneLinkType::Incoming ? 0 : (DestEscapeLane.SpanCount - 1);
						FZoneGraphEscapeLaneSpan& NextSpan = DestEscapeLane.Spans[NextSpanIndex];

						const float Danger = (Span.Danger + NextSpan.Danger) * 0.5f;
						const float Distance = FloatCastChecked<float>(FVector::Distance(Span.Position, NextSpan.Position), UE::LWC::DefaultFloatPrecision);
						const float NextCost = UE::ZoneGraphAnnotations::CalculateCost(Node.EscapeCost, Distance, Danger * DangerPenalty);
						if (NextCost < NextSpan.EscapeCost)
						{
							NextSpan.EscapeCost = NextCost;
							NextSpan.bReverseLaneDirection = ExitLinkType == EZoneLaneLinkType::Incoming;
							NextSpan.ExitLinkType = ExitLinkType;
							NextSpan.ExitLaneIndex = EscapeLane.LaneIndex;
							DisturbanceHeap.HeapPush(FEscapeNode(*DestEscapeLaneIndex, NextCost, NextSpanIndex, EscapeLane.LaneIndex, ExitLinkType));
						}
					}
				}
			}

			// Advance to adjacent
			UE::ZoneGraph::Query::GetLinkedLanes(*ZoneStorage, EscapeLane.LaneIndex, EZoneLaneLinkType::Adjacent, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, LinkedLanes);
			for (const FZoneGraphLinkedLane& LinkedLane : LinkedLanes)
			{
				const int32 DestLaneIndex = LinkedLane.DestLane.Index;
				// Dest lane must be part of the lanes to avoid.
				if (const int32* DestEscapeLaneIndex = EscapeGraph.LanesToEscapeLookup.Find(DestLaneIndex))
				{
					const FZoneLaneData& DestLane = ZoneStorage->Lanes[DestLaneIndex];
					const bool bIsDestEscapeLane = EscapeLaneTags.Pass(DestLane.Tags);

					// Allow Disturbance lanes to bleed into non-Disturbance lanes, and non-Disturbance lanes to non-Disturbance lanes too.
					// This allows to "drain" non-Disturbance lanes, but prevents Disturbanceing via non-Disturbance lanes.
					if (bIsCurrentEscapeLane || !bIsDestEscapeLane)
					{
						// Dest lane must be cheaper to reach via this lane.
						FZoneGraphEscapeLaneAction& DestEscapeLane = EscapeGraph.LanesToEscape[*DestEscapeLaneIndex];
						const uint8 NextSpanIndex = Node.ExitLinkType == EZoneLaneLinkType::Incoming ? (DestEscapeLane.SpanCount - 1) : 0;
						FZoneGraphEscapeLaneSpan& NextSpan = DestEscapeLane.Spans[NextSpanIndex];

						const float Danger = (Span.Danger + NextSpan.Danger) * 0.5f;
						const float Distance = FloatCastChecked<float>(FVector::Distance(Span.Position, NextSpan.Position), UE::LWC::DefaultFloatPrecision);
						const float NextCost = UE::ZoneGraphAnnotations::CalculateCost(Node.EscapeCost, Distance, Danger * DangerPenalty + AdjacentPenalty);
						if (NextCost < NextSpan.EscapeCost)
						{
							NextSpan.EscapeCost = NextCost;
							NextSpan.bReverseLaneDirection = false;
							NextSpan.ExitLinkType = EZoneLaneLinkType::Adjacent;
							NextSpan.ExitLaneIndex = EscapeLane.LaneIndex;
							DisturbanceHeap.HeapPush(FEscapeNode(*DestEscapeLaneIndex, NextCost, NextSpanIndex, EscapeLane.LaneIndex, EZoneLaneLinkType::Adjacent));
						}
					}
				}
			}
		}
	}

	// Find max escape cost (used for debug visualization).
	EscapeGraph.MaxEscapeCost = 0.0f;
	for (FZoneGraphEscapeLaneAction& EscapeLane : EscapeGraph.LanesToEscape)
	{
		for (uint8 SpanIndex = 0; SpanIndex < EscapeLane.SpanCount; SpanIndex++)
		{
			const FZoneGraphEscapeLaneSpan& Span = EscapeLane.Spans[SpanIndex];
			EscapeGraph.MaxEscapeCost = FMath::Max(EscapeGraph.MaxEscapeCost, Span.EscapeCost);
		}
	}
}

void UZoneGraphDisturbanceAnnotation::UpdateAnnotationTags(FZoneGraphAnnotationTagContainer& AnnotationTagContainer)
{
	// Remove flags from previous update
	for (FZoneGraphDataEscapeGraph& EscapeGraph : EscapeGraphs)
	{
		if (!EscapeGraph.PreviousLanes.IsEmpty())
		{
			TArrayView<FZoneGraphTagMask> LaneTags = AnnotationTagContainer.GetMutableAnnotationTagsForData(EscapeGraph.DataHandle);
			for (const int32 LaneIndex : EscapeGraph.PreviousLanes)
			{
				LaneTags[LaneIndex].Remove(PreviouslyAppliedTags);
			}
		}
		EscapeGraph.PreviousLanes.Reset();
	}
	PreviouslyAppliedTags = FZoneGraphTagMask::None;

	if (DangerAnnotationTag.IsValid() || ObstacleAnnotationTag.IsValid())
	{
		for (FZoneGraphDataEscapeGraph& EscapeGraph : EscapeGraphs)
		{
			if (EscapeGraph.bInUse && !EscapeGraph.LanesToEscape.IsEmpty())
			{
				FZoneGraphTagMask TagsAdded;
				
				// Apply tags
				TArrayView<FZoneGraphTagMask> LaneTags = AnnotationTagContainer.GetMutableAnnotationTagsForData(EscapeGraph.DataHandle);
				for (const FZoneGraphEscapeLaneAction& EscapeLane : EscapeGraph.LanesToEscape)
				{
					LaneTags[EscapeLane.LaneIndex].Add(EscapeLane.Tags);
					TagsAdded.Add(EscapeLane.Tags);
				}

				// Store which lanes were touched so that we can undo on next update.
				EscapeGraph.PreviousLanes.Reserve(EscapeGraph.LanesToEscape.Num());
				for (const FZoneGraphEscapeLaneAction& EscapeLane : EscapeGraph.LanesToEscape)
				{
					EscapeGraph.PreviousLanes.Add(EscapeLane.LaneIndex);
				}
				
				PreviouslyAppliedTags.Add(TagsAdded);
			}
		}
	}
}

void UZoneGraphDisturbanceAnnotation::HandleEvents(const FInstancedStructContainer& Events)
{
	for (FConstStructView Event : Events)
	{
		if (const FZoneGraphDisturbanceArea* Danger = Event.GetPtr<const FZoneGraphDisturbanceArea>())
		{
			FZoneGraphDisturbanceArea* ExistingDanger = Dangers.FindByPredicate([Danger](const FZoneGraphDisturbanceArea& Area) { return Area.InstigatorID == Danger->InstigatorID; });
			if (ExistingDanger)
			{
				*ExistingDanger = *Danger;
			}
			else
			{
				Dangers.Add(*Danger);
			}
			bDisturbancesChanged = true;
		}
		else if (const FZoneGraphObstacleDisturbanceArea* Obstacle = Event.GetPtr<const FZoneGraphObstacleDisturbanceArea>())
		{
			if (Obstacle->Action == EZoneGraphObstacleDisturbanceAreaAction::Add)
			{
				FZoneGraphObstacleDisturbanceArea* ExistingObstacle = Obstacles.FindByPredicate([Obstacle](const FZoneGraphObstacleDisturbanceArea& Area) { return Area.ObstacleID == Obstacle->ObstacleID; });
				if (ExistingObstacle)
				{
					*ExistingObstacle = *Obstacle;
				}
				else
				{
					Obstacles.Add(*Obstacle);
				}
			}
			else if (Obstacle->Action == EZoneGraphObstacleDisturbanceAreaAction::Remove)
			{
				Obstacles.Remove(*Obstacle);
			}
			bDisturbancesChanged = true;
		}
	}
}

void UZoneGraphDisturbanceAnnotation::PostZoneGraphDataAdded(const AZoneGraphData& ZoneGraphData)
{
	const FZoneGraphStorage& Storage = ZoneGraphData.GetStorage();
	const int32 Index = Storage.DataHandle.Index;
	
	if (Index >= EscapeGraphs.Num())
	{
		EscapeGraphs.SetNum(Index + 1);
	}

	FZoneGraphDataEscapeGraph& EscapeGraph = EscapeGraphs[Index];
	if (!EscapeGraph.bInUse)
	{
		EscapeGraph.LanesToEscape.Reset();
		EscapeGraph.PreviousLanes.Reset();
		EscapeGraph.DataHandle = Storage.DataHandle;
		EscapeGraph.bInUse = true;
	}
}

void UZoneGraphDisturbanceAnnotation::PreZoneGraphDataRemoved(const AZoneGraphData& ZoneGraphData)
{
	const FZoneGraphStorage& Storage = ZoneGraphData.GetStorage();
	const int32 Index = Storage.DataHandle.Index;

	if (!EscapeGraphs.IsValidIndex(Index))
	{
		return;
	}

	// Not removing the flags, expect that ZoneGraphAnnotationSubsystem cleans up when data is removed too.
	FZoneGraphDataEscapeGraph& EscapeGraph = EscapeGraphs[Index];
	EscapeGraph.LanesToEscape.Empty();
	EscapeGraph.PreviousLanes.Empty();
	EscapeGraph.LanesToEscapeLookup.Empty();
	EscapeGraph.DataHandle = FZoneGraphDataHandle();
	EscapeGraph.bInUse = false;
}

FZoneGraphTagMask UZoneGraphDisturbanceAnnotation::GetAnnotationTags() const
{
	FZoneGraphTagMask AnnotationTags;

	AnnotationTags.Add(DangerAnnotationTag);
	AnnotationTags.Add(ObstacleAnnotationTag);

	return AnnotationTags;
}

#if UE_ENABLE_DEBUG_DRAWING
void UZoneGraphDisturbanceAnnotation::DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy)
{
	const UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());
	const UZoneGraphAnnotationSubsystem* ZoneGraphAnnotationSubsystem = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(GetWorld());

	if (!ZoneGraph || !ZoneGraphAnnotationSubsystem)
	{
		return;
	}

	static const FVector ZOffset(0, 0, 25.0f);
	static const FLinearColor EscapeCostLowColor(FColor(170, 235, 239));
	static const FLinearColor EscapeCostHighColor(FColor(255, 61, 0));
	static constexpr float ArrowSize = 15.0f;

	FVector ViewLocation = FVector::ZeroVector;
	FRotator ViewRotation = FRotator::ZeroRotator;
	GetFirstViewPoint(ViewLocation, ViewRotation);

	LastDebugDrawLocation = ViewLocation; 
	
	const float DrawDistance = GetMaxDebugDrawDistance();
	const float DrawDistanceSq = FMath::Square(DrawDistance);
	const float LabelDrawDistanceSq = FMath::Square(DrawDistance * 0.15f);

	for (const FZoneGraphDataEscapeGraph& EscapeGraph : EscapeGraphs)
	{
		if (!EscapeGraph.bInUse)
		{
			continue;
		}
		
		if (const FZoneGraphStorage* ZoneStorage = ZoneGraph->GetZoneGraphStorage(EscapeGraph.DataHandle))
		{
			for (const FZoneGraphEscapeLaneAction& EscapeLane : EscapeGraph.LanesToEscape)
			{
				if (EscapeLane.SpanCount == 0)
				{
					continue;
				}
				const float DistanceSq = FloatCastChecked<float>(FVector::DistSquared(ViewLocation, EscapeLane.Spans[EscapeLane.SpanCount/2].Position), UE::LWC::DefaultFloatPrecision);
				if (DistanceSq > DrawDistanceSq)
				{
					continue;
				}

				const bool bAddLabel = DistanceSq < LabelDrawDistanceSq;
				
				const FZoneGraphLaneHandle LaneHandle(EscapeLane.LaneIndex, EscapeGraph.DataHandle);
				
				// Draw lane to indicate danger.
				FZoneGraphLaneLocation PrevLocation;
				UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneStorage, LaneHandle, 0, PrevLocation);

				const FZoneGraphTagMask TagsOnLane = ZoneGraphAnnotationSubsystem->GetAnnotationTags(LaneHandle);

				FString TagString;
				if (TagsOnLane.Contains(DangerAnnotationTag) )
				{
					TagString += "Danger ";	
				}

				if (TagsOnLane.Contains(ObstacleAnnotationTag))
				{
					TagString += "Obstacle ";
				}
				
				for (uint8 SpanIndex = 0; SpanIndex < EscapeLane.SpanCount; SpanIndex++)
				{
					const FZoneGraphEscapeLaneSpan& Span = EscapeLane.Spans[SpanIndex];
					FZoneGraphLaneLocation SplitLocation;
					UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneStorage, LaneHandle, Span.SplitDistance, SplitLocation);

					const FLinearColor DangerColor = FMath::Lerp(EscapeCostLowColor, EscapeCostHighColor, FMath::Square(Span.Danger));
					const FLinearColor MarkerColor = DangerColor * 0.5f;

					// Draw lane segment that belongs to the span.
					UE::ZoneGraph::RenderingUtilities::AppendLane(DebugProxy, *ZoneStorage, PrevLocation, SplitLocation, DangerColor.ToFColor(/*sRGB*/true), 4.0f, ZOffset);

					if (SpanIndex < EscapeLane.SpanCount - 1)
					{
						// Draw span separating line.
						DebugProxy->Lines.Emplace(SplitLocation.Position + ZOffset, SplitLocation.Position + FVector::UpVector * ArrowSize + ZOffset, MarkerColor.ToFColor(/*sRGB*/true), 2.0f);
					}

					// Draw direction arrow
					const FVector ArrowPosition = Span.Position + ZOffset * 1.1f;
					const FVector ArrowDirection = Span.bReverseLaneDirection ? -Span.Direction : Span.Direction;
					const FVector ArrowSide = FVector::CrossProduct(Span.Direction, FVector::UpVector);
					if (Span.ExitLinkType == EZoneLaneLinkType::Adjacent)
					{
						DebugProxy->Lines.Emplace(ArrowPosition - ArrowSide * ArrowSize, ArrowPosition + ArrowSide * ArrowSize, MarkerColor.ToFColor(/*sRGB*/true), 2.0f);
					}
					else
					{
						DebugProxy->Lines.Emplace(ArrowPosition - ArrowDirection * ArrowSize - ArrowSide * ArrowSize, ArrowPosition + ArrowDirection * ArrowSize, MarkerColor.ToFColor(/*sRGB*/true), 2.0f);
						DebugProxy->Lines.Emplace(ArrowPosition - ArrowDirection * ArrowSize + ArrowSide * ArrowSize, ArrowPosition + ArrowDirection * ArrowSize, MarkerColor.ToFColor(/*sRGB*/true), 2.0f);
					}

					if (bAddLabel)
					{
						if (Span.ExitLaneIndex == INDEX_NONE)
						{
							DebugProxy->Texts.Emplace(FString::Printf(TEXT("ERROR!")), ArrowPosition + FVector::UpVector * ArrowSize, FColor::Red);
						}
						else
						{
							DebugProxy->Texts.Emplace(FString::Printf(TEXT("%d->%d\nescapeCost: %.1f\ndanger: %.1f\ntags: %s"), EscapeLane.LaneIndex, Span.ExitLaneIndex, Span.EscapeCost, Span.Danger, *TagString), ArrowPosition + FVector::UpVector * ArrowSize, FColor::Red);
						}
					}

					if (Span.bLeadsToExit)
					{
						const FVector Point = SpanIndex == 0 ? PrevLocation.Position : SplitLocation.Position + ZOffset;
						DebugProxy->Lines.Emplace(Point, Point + FVector::UpVector * ArrowSize, FColor::Blue, 2.0f);
						if (bAddLabel)
						{
							DebugProxy->Texts.Emplace(FString(TEXT("EXIT")), Point + FVector::UpVector * ArrowSize, FColor::Blue);
						}
					}
					
					PrevLocation = SplitLocation;
				}
			}
		}
	}

	for (const FZoneGraphDisturbanceArea& Danger : Dangers)
	{
		const float DistanceSq = FloatCastChecked<float>(FVector::DistSquared(ViewLocation, Danger.Position), UE::LWC::DefaultFloatPrecision);
		if (DistanceSq > DrawDistanceSq)
		{
			continue;
		}

		const FBox QueryBounds = FBox::BuildAABB(Danger.Position, FVector(Danger.Radius));
		DebugProxy->Boxes.Emplace(QueryBounds, FColor::Red);
		DebugProxy->Lines.Emplace(Danger.Position - FVector(ArrowSize, 0, 0), Danger.Position + FVector(ArrowSize, 0, 0), FColor::Red, 2.0f);
		DebugProxy->Lines.Emplace(Danger.Position - FVector(0, ArrowSize, 0), Danger.Position + FVector(0, ArrowSize, 0), FColor::Red, 2.0f);
		DebugProxy->Texts.Emplace(FString(TEXT("DANGER!")), Danger.Position, FColor::Red);
	}

	for (const FZoneGraphObstacleDisturbanceArea& Obstacle : Obstacles)
	{
		const float DistanceSq = FloatCastChecked<float>(FVector::DistSquared(ViewLocation, Obstacle.Position), UE::LWC::DefaultFloatPrecision);
		if (DistanceSq > DrawDistanceSq)
		{
			continue;
		}

		const FColor ObstacleColor = FColor::Blue;
		const FBox QueryBounds = FBox::BuildAABB(Obstacle.Position, FVector(Obstacle.Radius));
		DebugProxy->Boxes.Emplace(QueryBounds, ObstacleColor);
		DebugProxy->Lines.Emplace(Obstacle.Position - FVector(ArrowSize, 0, 0), Obstacle.Position + FVector(ArrowSize, 0, 0), ObstacleColor, 2.0f);
		DebugProxy->Lines.Emplace(Obstacle.Position - FVector(0, ArrowSize, 0), Obstacle.Position + FVector(0, ArrowSize, 0), ObstacleColor, 2.0f);
		DebugProxy->Texts.Emplace(FString(TEXT("OBSTACLE")), Obstacle.Position, ObstacleColor);
		DebugProxy->Spheres.Emplace(Obstacle.ObstacleRadius, Obstacle.Position, ObstacleColor);
	}
}

void UZoneGraphDisturbanceAnnotation::DebugDrawCanvas(UCanvas* Canvas, APlayerController*)
{
	const FColor OldDrawColor = Canvas->DrawColor;
	const FFontRenderInfo FontInfo = Canvas->CreateFontRenderInfo(true, true);
	const UFont* RenderFont = GEngine->GetSmallFont();
	const float LineHeight = RenderFont->GetMaxCharHeight() * 1.2f;

	Canvas->SetDrawColor(FColor::White);

	constexpr float PosX = 40;
	float PosY = 200;

	const int32 NumLines = FMath::Min(Dangers.Num(), 15);
	
	for (int32 Index = 0; Index < NumLines; Index++)
	{
		const FZoneGraphDisturbanceArea& Danger = Dangers[Index];
		FString Text = FString::Printf(TEXT("Danger %.1f"), Danger.Duration);
		
		Canvas->SetDrawColor(FColor::Red);
		Canvas->DrawText(RenderFont, Text, PosX, PosY, 1, 1, FontInfo);

		PosY -= LineHeight;
	}
	
	if (NumLines < Dangers.Num())
	{
		FString Text = FString::Printf(TEXT("%d more dangers..."), Dangers.Num() - NumLines);
		Canvas->SetDrawColor(FColor::Red);
		Canvas->DrawText(RenderFont, Text, PosX, PosY, 1, 1, FontInfo);
	}

	Canvas->SetDrawColor(OldDrawColor);
}
#endif // UE_ENABLE_DEBUG_DRAWING


//////////////////////////////////////////////////////////////////////////
// UZoneGraphDisturbanceAnnotationTest

#if UE_ENABLE_DEBUG_DRAWING
FBox UZoneGraphDisturbanceAnnotationTest::CalcBounds(const FTransform& LocalToWorld) const
{
	const FVector Size(DangerRadius);
	const FBox CompBounds(-Size, Size);
	return CompBounds.TransformBy(LocalToWorld);
}

void UZoneGraphDisturbanceAnnotationTest::DebugDraw(FDebugRenderSceneProxy* DebugProxy)
{
	check(OwnerComponent);
	
	const FVector CenterSize(10);
	const FVector DangerSize(DangerRadius);
	const FVector Position = OwnerComponent->GetComponentTransform().GetLocation() + Offset;

	DebugProxy->Boxes.Emplace(FBox(Position - CenterSize, Position + CenterSize), FColor::Red);
	DebugProxy->Boxes.Emplace(FBox(Position - DangerSize, Position + DangerSize), FColor::Red);
}
#endif // UE_ENABLE_DEBUG_DRAWING

void UZoneGraphDisturbanceAnnotationTest::Trigger()
{
	UZoneGraphAnnotationSubsystem* ZoneGraphAnnotation = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(GetWorld());
	if (!ZoneGraphAnnotation)
	{
		return;
	}
	check(OwnerComponent);

	// This is just for testing, will add later ability to add multiple tests ala ZoneGraphTesting actor.
	FZoneGraphDisturbanceArea Danger;
	Danger.Position = OwnerComponent->GetComponentTransform().GetLocation() + Offset;
	Danger.Radius = DangerRadius;
	Danger.Duration = Duration;
	Danger.InstigatorID = PointerHash(this);
	
	ZoneGraphAnnotation->SendEvent(Danger);
}

#ifdef _MSC_VER
#pragma warning(pop)
#endif
