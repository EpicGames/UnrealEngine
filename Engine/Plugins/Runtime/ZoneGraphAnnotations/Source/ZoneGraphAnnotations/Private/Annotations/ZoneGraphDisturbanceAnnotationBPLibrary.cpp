// Copyright Epic Games, Inc. All Rights Reserved.

#include "Annotations/ZoneGraphDisturbanceAnnotationBPLibrary.h"
#include "Annotations/ZoneGraphDisturbanceAnnotation.h"
#include "ZoneGraphAnnotationSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphDisturbanceAnnotationBPLibrary)

void UZoneGraphDisturbanceAnnotationBPLibrary::TriggerDanger(UObject* WorldContextObject, const AActor* Instigator, const FVector Position, const float Radius, const float Duration)
{
	if (!WorldContextObject)
	{
		UE_LOGF(LogZoneGraphAnnotations, Error, "%ls: WorldContextObject not set.", ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}
	
	UZoneGraphAnnotationSubsystem* ZoneGraphAnnotation = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(WorldContextObject->GetWorld());
	if (!ZoneGraphAnnotation)
	{
		UE_LOGF(LogZoneGraphAnnotations, Error, "%ls: Expecting ZoneGraphAnnotationSubsystem to be present.", ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}

	// This is just for testing, will add later ability to add multiple tests ala ZoneGraphTesting actor.
	FZoneGraphDisturbanceArea Danger;
	Danger.Position = Position;
	Danger.Radius = Radius;
	Danger.Duration = Duration;
	Danger.InstigatorID = Instigator != nullptr ? PointerHash(Instigator) : 0;
	
	ZoneGraphAnnotation->SendEvent(Danger);
}
