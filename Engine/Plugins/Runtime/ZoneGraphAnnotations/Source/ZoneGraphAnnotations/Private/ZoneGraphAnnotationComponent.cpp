// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphAnnotationComponent.h"
#include "PrimitiveViewRelevance.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphSettings.h"
#include "Debug/DebugDrawService.h"
#include "Engine/DebugCameraController.h"
#include "Engine/LocalPlayer.h"
#include "UnrealEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphAnnotationComponent)

#if UE_ENABLE_DEBUG_DRAWING
//////////////////////////////////////////////////////////////////////////
// FZoneGraphAnnotationSceneProxy

FZoneGraphAnnotationSceneProxy::FZoneGraphAnnotationSceneProxy(const UPrimitiveComponent& InComponent, const EDrawType InDrawType)
	: FDebugRenderSceneProxy(&InComponent)
{
	DrawType = InDrawType;
	ViewFlagName = TEXT("ZoneGraph");
	ViewFlagIndex = uint32(FEngineShowFlags::FindIndexByName(*ViewFlagName));
}

SIZE_T FZoneGraphAnnotationSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FPrimitiveViewRelevance FZoneGraphAnnotationSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && ViewFlagIndex != INDEX_NONE && View->Family->EngineShowFlags.GetSingleFlag(ViewFlagIndex);
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = true;
	return Result;
}

uint32 FZoneGraphAnnotationSceneProxy::GetMemoryFootprint(void) const
{
	return sizeof(*this) + FDebugRenderSceneProxy::GetAllocatedSize();
}

#endif // UE_ENABLE_DEBUG_DRAWING


//////////////////////////////////////////////////////////////////////////
// UZoneGraphAnnotationComponent
UZoneGraphAnnotationComponent::UZoneGraphAnnotationComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionEnabled(ECollisionEnabled::NoCollision);

#if WITH_EDITORONLY_DATA
	HitProxyPriority = HPP_Wireframe;
#endif
}

#if WITH_EDITOR
void UZoneGraphAnnotationComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	// Trigger tag registering when properties change.
	UWorld* World = GetWorld();
    if (World == nullptr)
    {
    	return;
    }

	if (UZoneGraphAnnotationSubsystem* ZoneGraphAnnotation = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(World))
	{
		ZoneGraphAnnotation->ReregisterTagsInEditor();
	}
}
#endif

void UZoneGraphAnnotationComponent::OnRegister()
{
	Super::OnRegister();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	UWorld* World = GetWorld();

#if WITH_EDITOR
	// Do not process any component registered to preview world
	if (World && World->WorldType == EWorldType::EditorPreview)
	{
		return;
	}
#endif

	if (UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(World))
	{
		PostSubsystemsInitialized();
	}
	else
	{
		OnPostWorldInitDelegateHandle = FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UZoneGraphAnnotationComponent::OnPostWorldInit);
	}

#if UE_ENABLE_DEBUG_DRAWING
	CanvasDebugDrawDelegateHandle = UDebugDrawService::Register(TEXT("ZoneGraph"), FDebugDrawDelegate::CreateUObject(this, &UZoneGraphAnnotationComponent::DebugDrawCanvas));
#endif
}

void UZoneGraphAnnotationComponent::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues)
{
	if (World == GetWorld())
	{
		PostSubsystemsInitialized();
	}

	FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitDelegateHandle);
}

void UZoneGraphAnnotationComponent::PostSubsystemsInitialized()
{
	UWorld* World = GetWorld();
	check(World);

	if (UZoneGraphAnnotationSubsystem* ZoneGraphAnnotation = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(World))
	{
		ZoneGraphAnnotation->RegisterAnnotationComponent(*this);
	}

	// Add the zonegraph data that already exists in the system
	if (const UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(World))
	{
		for (const FRegisteredZoneGraphData& Registered : ZoneGraph->GetRegisteredZoneGraphData())
		{
			if (Registered.bInUse && Registered.ZoneGraphData)
			{
				PostZoneGraphDataAdded(*Registered.ZoneGraphData);
			}
		}

		OnPostZoneGraphDataAddedHandle = UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.AddUObject(this, &UZoneGraphAnnotationComponent::OnPostZoneGraphDataAdded);
		OnPreZoneGraphDataRemovedHandle = UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.AddUObject(this, &UZoneGraphAnnotationComponent::OnPreZoneGraphDataRemoved);
	}
}

void UZoneGraphAnnotationComponent::OnUnregister()
{
	Super::OnUnregister();

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

#if WITH_EDITOR
	// Do not process any component registered to preview world
	if (World->WorldType == EWorldType::EditorPreview)
	{
		return;
	}
#endif

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (UZoneGraphAnnotationSubsystem* ZoneGraphAnnotation = UWorld::GetSubsystem<UZoneGraphAnnotationSubsystem>(World))
	{
		ZoneGraphAnnotation->UnregisterAnnotationComponent(*this);
	}

	UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.Remove(OnPostZoneGraphDataAddedHandle);
	UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.Remove(OnPreZoneGraphDataRemovedHandle);

#if UE_ENABLE_DEBUG_DRAWING
	UDebugDrawService::Unregister(CanvasDebugDrawDelegateHandle);
#endif
}

void UZoneGraphAnnotationComponent::OnPostZoneGraphDataAdded(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	PostZoneGraphDataAdded(*ZoneGraphData);
}

void UZoneGraphAnnotationComponent::OnPreZoneGraphDataRemoved(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	PreZoneGraphDataRemoved(*ZoneGraphData);
}

#if UE_ENABLE_DEBUG_DRAWING
void UZoneGraphAnnotationComponent::GetFirstViewPoint(FVector& ViewLocation, FRotator& ViewRotation) const
{
	ViewLocation = FVector::ZeroVector;
	ViewRotation = FRotator::ZeroRotator;

	bool bFound = false;
	
	if (const UWorld* World = GetWorld())
	{
		// Now go through all current player controllers and add if they do not exist
		for (FConstPlayerControllerIterator PlayerIterator = World->GetPlayerControllerIterator(); PlayerIterator; ++PlayerIterator)
		{
			if (const APlayerController* PlayerController = (*PlayerIterator).Get())
			{
				PlayerController->GetPlayerViewPoint(ViewLocation, ViewRotation);
				bFound = true;
				break;
			}
		}

		// spectator mode
		if (!bFound)
		{
			for (FLocalPlayerIterator It(GEngine, const_cast<UWorld*>(World)); It; ++It)
			{
				ADebugCameraController* SpectatorPC = Cast<ADebugCameraController>(It->PlayerController);
				if (SpectatorPC)
				{
					SpectatorPC->GetPlayerViewPoint(ViewLocation, ViewRotation);
					bFound = true;
					break;
				}
			}
		}

	}
}

float UZoneGraphAnnotationComponent::GetMaxDebugDrawDistance() const
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	return ZoneGraphSettings ? ZoneGraphSettings->GetShapeMaxDrawDistance() : 0.0f;
}

FDebugRenderSceneProxy* UZoneGraphAnnotationComponent::CreateDebugSceneProxy()
{
	FZoneGraphAnnotationSceneProxy* DebugProxy = new FZoneGraphAnnotationSceneProxy(*this);
	if (bEnableDebugDrawing)
	{
		DebugDraw(DebugProxy);
	}
	return DebugProxy;
}
#endif // UE_ENABLE_DEBUG_DRAWING

FBoxSphereBounds UZoneGraphAnnotationComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox(ForceInit);

	if (const UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
	{
		BoundingBox = ZoneGraph->GetCombinedBounds();
	}
	
	return FBoxSphereBounds(BoundingBox);
}
