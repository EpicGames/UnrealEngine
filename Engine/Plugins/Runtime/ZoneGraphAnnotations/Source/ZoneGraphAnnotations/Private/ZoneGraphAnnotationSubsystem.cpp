// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphAnnotationComponent.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphSettings.h"
#include "MassSubsystemBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphAnnotationSubsystem)


UZoneGraphAnnotationSubsystem::UZoneGraphAnnotationSubsystem()
{
	TagToAnnotationLookup.Init(nullptr, static_cast<int32>(EZoneGraphTags::MaxTags));
}

void UZoneGraphAnnotationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	constexpr int32 InitialEventBufferSize = 1024;
	Events[0].ReserveBytes(InitialEventBufferSize);
	Events[1].ReserveBytes(InitialEventBufferSize);
	
	const UZoneGraphSubsystem* ZoneGraph = Collection.InitializeDependency<UZoneGraphSubsystem>();

	// Register existing data.
	int32 Index = 0;
	for (const FRegisteredZoneGraphData& Registered : ZoneGraph->GetRegisteredZoneGraphData())
	{
		if (Registered.bInUse && Registered.ZoneGraphData)
		{
			PostZoneGraphDataAdded(Registered.ZoneGraphData);
		}
		Index++;
	}

	OnPostZoneGraphDataAddedHandle = UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.AddUObject(this, &UZoneGraphAnnotationSubsystem::PostZoneGraphDataAdded);
	OnPreZoneGraphDataRemovedHandle = UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.AddUObject(this, &UZoneGraphAnnotationSubsystem::PreZoneGraphDataRemoved);

	// Note that we're using GetClass() rather than StaticClass() to work as expected for child-classes as well.
	// Child class can always override the traits registered this way.
	UE::Mass::Subsystems::RegisterSubsystemType(Collection, GetClass(), UE::Mass::FSubsystemTypeTraits::Make<UZoneGraphAnnotationSubsystem>());
}

void UZoneGraphAnnotationSubsystem::RegisterAnnotationComponent(UZoneGraphAnnotationComponent& Component)
{
	FRegisteredZoneGraphAnnotation& Registered = RegisteredComponents.AddDefaulted_GetRef();
	Registered.AnnotationComponent = &Component;
	Registered.AnnotationTags = Component.GetAnnotationTags();
	AddToAnnotationLookup(Component, Registered.AnnotationTags);
}

void UZoneGraphAnnotationSubsystem::UnregisterAnnotationComponent(UZoneGraphAnnotationComponent& Component)
{
	RemoveFromAnnotationLookup(Component);
	RegisteredComponents.RemoveAll([&Component](const FRegisteredZoneGraphAnnotation& Registered) { return Registered.AnnotationComponent == &Component; });
}

void UZoneGraphAnnotationSubsystem::Deinitialize()
{
	UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.Remove(OnPostZoneGraphDataAddedHandle);
	UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.Remove(OnPreZoneGraphDataRemovedHandle);

	Super::Deinitialize();
}

void UZoneGraphAnnotationSubsystem::AddToAnnotationLookup(UZoneGraphAnnotationComponent& Annotation, const FZoneGraphTagMask AnnotationTags)
{
	const uint32 Start = FMath::CountTrailingZeros(AnnotationTags.GetValue());
	const uint32 End = 32 - FMath::CountLeadingZeros(AnnotationTags.GetValue());
	for (uint32 Index = Start; Index < End; Index++)
	{
		check(Index <= (uint32)MAX_uint8);
		const FZoneGraphTag BitTag((uint8)Index);
		if (AnnotationTags.Contains(BitTag))
		{
			if (ensureMsgf(TagToAnnotationLookup[Index] == nullptr, TEXT("Annotation at index %d is already set to %s."), Index, *GetNameSafe(TagToAnnotationLookup[Index])) &&
				ensureMsgf(!AnnotationTagContainer.CombinedStaticTags.Contains(BitTag), TEXT("Annotation %s at index %d is using tag '%s' 'reserved for static lane data."),
					*GetNameSafe(TagToAnnotationLookup[Index]), Index,*UE::ZoneGraph::Helpers::GetTagName(BitTag).ToString()))
			{
				TagToAnnotationLookup[Index] = &Annotation;
			}
		}
	}
}

void UZoneGraphAnnotationSubsystem::RemoveFromAnnotationLookup(UZoneGraphAnnotationComponent& Annotation)
{
	for (int32 Index = 0; Index < TagToAnnotationLookup.Num(); Index++)
	{
		if (TagToAnnotationLookup[Index] == &Annotation)
		{
			TagToAnnotationLookup[Index] = nullptr;
		}
	}
}

#if WITH_EDITOR
void UZoneGraphAnnotationSubsystem::ReregisterTagsInEditor()
{
	// Reset and re-register.
	TagToAnnotationLookup.Init(nullptr, static_cast<int32>(EZoneGraphTags::MaxTags));

	for (FRegisteredZoneGraphAnnotation& Registered : RegisteredComponents)
	{
		if (Registered.AnnotationComponent != nullptr)
		{
			Registered.AnnotationTags = Registered.AnnotationComponent->GetAnnotationTags();
			AddToAnnotationLookup(*Registered.AnnotationComponent, Registered.AnnotationTags);
		}
	}
}
#endif // WITH_EDITOR

void UZoneGraphAnnotationSubsystem::PostZoneGraphDataAdded(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	const FZoneGraphStorage& Storage = ZoneGraphData->GetStorage();
	const int32 Index = Storage.DataHandle.Index;
	
	if (Index >= AnnotationTagContainer.DataAnnotationTags.Num())
	{
		AnnotationTagContainer.DataAnnotationTags.SetNum(Index + 1);
	}
	
	FZoneGraphDataAnnotationTags& AnnotationTags = AnnotationTagContainer.DataAnnotationTags[Index];
	AnnotationTags.LaneTags.Init(FZoneGraphTagMask::None, Storage.Lanes.Num());
	for (int32 LaneIndex = 0; LaneIndex < Storage.Lanes.Num(); LaneIndex++)
	{
		// Initialize with static tags
		const FZoneGraphTagMask LaneTags = Storage.Lanes[LaneIndex].Tags;
		AnnotationTags.LaneTags[LaneIndex] = LaneTags;

		// Add current lane static tags to the combined tags
		AnnotationTagContainer.CombinedStaticTags.Add(LaneTags);
	}
	AnnotationTags.DataHandle = Storage.DataHandle;
	AnnotationTags.bInUse = true;
}
	
void UZoneGraphAnnotationSubsystem::PreZoneGraphDataRemoved(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	const FZoneGraphStorage& Storage = ZoneGraphData->GetStorage();
	const int32 DataIndex = Storage.DataHandle.Index;

	if (!AnnotationTagContainer.DataAnnotationTags.IsValidIndex(DataIndex))
	{
		return;
	}
	
	FZoneGraphDataAnnotationTags& AnnotationTags = AnnotationTagContainer.DataAnnotationTags[DataIndex];
	AnnotationTags.LaneTags.Empty();
	AnnotationTags.DataHandle = FZoneGraphDataHandle();
	AnnotationTags.bInUse = false;
}

void UZoneGraphAnnotationSubsystem::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	// note that it's safe to store the value of CurrentEventStream since the code below is the only place that changes it.
	const int32 PrevEventStream = CurrentEventStream;
	{
		// do a thread-safe buffer switch first, then we can do whatever we want to the events
		UE::TScopeLock ScopeLock(EventsLock);

		// switch exposed current stream, so that event handling can add more events without disturbing the processing.
		CurrentEventStream ^= 1;
	}

	FInstancedStructContainer& CurrentEvents = Events[PrevEventStream];
	if (CurrentEvents.Num() > 0)
	{
		// Process requests
		for (const FRegisteredZoneGraphAnnotation& Registered : RegisteredComponents)
		{
			if (Registered.AnnotationComponent != nullptr)
			{
				Registered.AnnotationComponent->HandleEvents(CurrentEvents);
			}
		}

		CurrentEvents.Reset();
	}

	// Updates Annotations
	for (const FRegisteredZoneGraphAnnotation& Registered : RegisteredComponents)
	{
		if (Registered.AnnotationComponent != nullptr)
		{
			Registered.AnnotationComponent->TickAnnotation(DeltaTime, AnnotationTagContainer);
		}
	}
}

TStatId UZoneGraphAnnotationSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UZoneGraphAnnotationSubsystem, STATGROUP_Tickables);
}
