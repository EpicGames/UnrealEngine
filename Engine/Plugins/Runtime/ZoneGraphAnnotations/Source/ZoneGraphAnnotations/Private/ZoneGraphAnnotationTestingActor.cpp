// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphAnnotationTestingActor.h"
#include "ZoneGraphAnnotationComponent.h"
#include "Debug/DebugDrawService.h"

/////////////////////////////////////////////////////
// UZoneGraphAnnotationTestingComponent

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphAnnotationTestingActor)

UZoneGraphAnnotationTestingComponent::UZoneGraphAnnotationTestingComponent(const FObjectInitializer& ObjectInitialize)
	: Super(ObjectInitialize)
{
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
	
#if WITH_EDITORONLY_DATA
	HitProxyPriority = HPP_Wireframe;
#endif
}

FBoxSphereBounds UZoneGraphAnnotationTestingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox CombinedBounds(ForceInit);
	
#if UE_ENABLE_DEBUG_DRAWING
	for (const UZoneGraphAnnotationTest* Test : Tests)
	{
		if (Test != nullptr)
		{
			CombinedBounds += Test->CalcBounds(LocalToWorld);
		}
	}
#endif

	return CombinedBounds;
}

void UZoneGraphAnnotationTestingComponent::Trigger()
{
	for (UZoneGraphAnnotationTest* Test : Tests)
	{
		if (Test != nullptr)
		{
			Test->Trigger();
		}
	}
}

#if WITH_EDITOR
void UZoneGraphAnnotationTestingComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	MarkRenderStateDirty();
}
#endif

void UZoneGraphAnnotationTestingComponent::OnRegister()
{
	Super::OnRegister();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		for (UZoneGraphAnnotationTest* Test : Tests)
		{
			if (Test != nullptr)
			{
				Test->SetOwner(this);
			}
		}
	
#if UE_ENABLE_DEBUG_DRAWING
		CanvasDebugDrawDelegateHandle = UDebugDrawService::Register(TEXT("ZoneGraph"), FDebugDrawDelegate::CreateUObject(this, &UZoneGraphAnnotationTestingComponent::DebugDrawCanvas));
#endif
	}
}

void UZoneGraphAnnotationTestingComponent::OnUnregister()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if UE_ENABLE_DEBUG_DRAWING
		UDebugDrawService::Unregister(CanvasDebugDrawDelegateHandle);
#endif
	}
	
	Super::OnUnregister();
}

#if UE_ENABLE_DEBUG_DRAWING
FDebugRenderSceneProxy* UZoneGraphAnnotationTestingComponent::CreateDebugSceneProxy()
{
	FZoneGraphAnnotationSceneProxy* DebugProxy = new FZoneGraphAnnotationSceneProxy(*this, FZoneGraphAnnotationSceneProxy::WireMesh);
	DebugDraw(DebugProxy);
	return DebugProxy;
}

void UZoneGraphAnnotationTestingComponent::DebugDraw(FDebugRenderSceneProxy* DebugProxy)
{
	for (UZoneGraphAnnotationTest* Test : Tests)
	{
		if (Test != nullptr)
		{
			Test->DebugDraw(DebugProxy);
		}
	}
}

void UZoneGraphAnnotationTestingComponent::DebugDrawCanvas(UCanvas* Canvas, APlayerController* PlayerController)
{
	for (UZoneGraphAnnotationTest* Test : Tests)
	{
		if (Test != nullptr)
		{
			Test->DebugDrawCanvas(Canvas, PlayerController);
		}
	}
}
#endif // UE_ENABLE_DEBUG_DRAWING

/////////////////////////////////////////////////////
// AZoneGraphAnnotationTestingActor

AZoneGraphAnnotationTestingActor::AZoneGraphAnnotationTestingActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TestingComp = CreateDefaultSubobject<UZoneGraphAnnotationTestingComponent>(TEXT("TestingComp"));
	RootComponent = TestingComp;

	SetHidden(true);
	SetCanBeDamaged(false);
}

void AZoneGraphAnnotationTestingActor::Trigger()
{
	if (TestingComp != nullptr)
	{
		TestingComp->Trigger();
	}
}

#if WITH_EDITOR
void AZoneGraphAnnotationTestingActor::PostEditMove(bool bFinished)
{
	if (TestingComp != nullptr)
	{
		TestingComp->MarkRenderStateDirty();
	}
}
#endif
