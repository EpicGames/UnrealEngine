// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphAnnotationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphAnnotationTypes)

DEFINE_LOG_CATEGORY(LogZoneGraphAnnotations);

const FMassLaneObstacleID FMassLaneObstacleID::InvalidID;
uint64 FMassLaneObstacleID::NextUniqueID = 0;

