// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ZoneGraphAnnotationComponent.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "ZoneGraphAnnotationTestingActor.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphDisturbanceAnnotation.generated.h"

#define UE_API ZONEGRAPHANNOTATIONS_API

class UZoneGraphSubsystem;

/** Event for indicating an area of disturbance. */
USTRUCT()
struct FZoneGraphDisturbanceArea : public FZoneGraphAnnotationEventBase
{
	GENERATED_BODY()

	/** Center of the area. */
	UPROPERTY()
	FVector Position = FVector::ZeroVector;

	/** Radius of the effect. */
	UPROPERTY()
	float Radius = 0.0f;

	/** Duration of the danger. */
	UPROPERTY()
	float Duration = 0.0f;

	/** ID of the instigator of this event, events from the same instigator are combined. */
	UPROPERTY()
	uint32 InstigatorID = 0;
};

/** Actions for disturbances. */
UENUM()
enum class EZoneGraphObstacleDisturbanceAreaAction : uint8
{
	Add,	// Add
	Remove	// Remove
};

/** Event for indicating an obstacle. */
USTRUCT()
struct FZoneGraphObstacleDisturbanceArea : public FZoneGraphAnnotationEventBase
{
	GENERATED_BODY()

	bool operator==(const FZoneGraphObstacleDisturbanceArea& Other) const { return ObstacleID == Other.ObstacleID; } 
	
	/** Center of the obstacle. */
	UPROPERTY()
	FVector Position = FVector::ZeroVector;

	/** Radius of the effect of the disturbance. */
	UPROPERTY()
	float Radius = 0.f;

	/** Radius of the obstacle. */
	UPROPERTY()
	float ObstacleRadius = 0.f;
	
	/** ID. */
	FMassLaneObstacleID ObstacleID;
	
	/** Disturbance event action (ex: add/remove). */
	UPROPERTY()
	EZoneGraphObstacleDisturbanceAreaAction Action = EZoneGraphObstacleDisturbanceAreaAction::Add;
};

/** Instructions how to escape from one half of a lane. */
struct FZoneGraphEscapeLaneSpan
{
	/** Representative position of the action, used during graph calculation and debug drawing. */
	FVector Position = FVector::ZeroVector;
	/** Lane direction, used for debug drawing. */
	FVector Direction = FVector::ZeroVector;
	/** Distance along the lane where lane is split in 2 spans. */
	float SplitDistance = 0.0f;
	/** Max danger value from all danger locations. */
	float Danger = 0.0f;
	/** Disturbance cost value, lower the value leads to safety. */
	float EscapeCost = 0.0f;
	/** Index to exit lane within current ZoneGraph data */
	int32 ExitLaneIndex = INDEX_NONE;
	/** True if the Disturbance action should move backwards along the lane. */
	bool bReverseLaneDirection = false;
	/** True if the action leads to exit */
	bool bLeadsToExit = false;
	/** Describes what type of linked lane to follow. */
	EZoneLaneLinkType ExitLinkType = EZoneLaneLinkType::None;
};

/** Data describing how to Disturbance a lane, the lane is split in half and each half has its' own Disturbance action and direction. */
struct FZoneGraphEscapeLaneAction
{
	FZoneGraphEscapeLaneAction() = default;
	FZoneGraphEscapeLaneAction(const int32 InLaneIndex) : LaneIndex(InLaneIndex) {}

	static constexpr uint8 MaxSpans = 8;

	/** Finds a span based on the split distances. */
	uint8 FindSpanIndex(const float Distance) const
	{
		for (uint8 SpanIndex = 0; SpanIndex < SpanCount; SpanIndex++)
		{
			if (Distance < Spans[SpanIndex].SplitDistance)
			{
				return SpanIndex;
			}
		}
		return SpanCount - 1;
	}
	
	/** Lane Index the data belongs to. */
	int32 LaneIndex = 0;
	/** Cached lane length from ZoneGraph. */
	float LaneLength = 0.0f;
	/** Disturbance annotation tags */
	FZoneGraphTagMask Tags;
	/** Disturbance actions for Start (0), and End (1) spans of the lane. */
	FZoneGraphEscapeLaneSpan Spans[MaxSpans];
	/** Number of spans on this lane */
	uint8 SpanCount = 0;
};

/** PerZoneGraphData escape graph. */
struct FZoneGraphDataEscapeGraph
{
	/** Handle of the data this Disturbance data relates to */ 
	FZoneGraphDataHandle DataHandle;
	/** True, if this entry is in use. */
	bool bInUse = false;
	
	/** Array of lanes to avoid. */
	TArray<FZoneGraphEscapeLaneAction> LanesToEscape;
	/** Array of previous lanes that were marked. */
	TArray<int32> PreviousLanes;
	/** Lookup from ZoneGraph Lane index to index in LanesToEscape */
	TMap<int32, int32> LanesToEscapeLookup;
	/** Max escape cost, used for visualization */
	float MaxEscapeCost = 0.0f;
};


/**
 * ZoneGraph Disturbance Annotation
 */
UCLASS(MinimalAPI, ClassGroup = AI, BlueprintType, meta = (BlueprintSpawnableComponent))
class UZoneGraphDisturbanceAnnotation : public UZoneGraphAnnotationComponent
{
	GENERATED_BODY()

public:
	UE_API UZoneGraphDisturbanceAnnotation(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** @return Disturbance action for a specific lane, or null of the lane does not have Disturbance action. */
	const FZoneGraphEscapeLaneAction* GetEscapeAction(const FZoneGraphLaneHandle LaneHandle) const
	{
		const FZoneGraphDataEscapeGraph& EscapeGraph = EscapeGraphs[LaneHandle.DataHandle.Index];
		const int32* Index = EscapeGraph.LanesToEscapeLookup.Find(LaneHandle.Index);
		return Index != nullptr ? &EscapeGraph.LanesToEscape[*Index] : nullptr;
	}
	
protected:

	UE_API virtual void PostSubsystemsInitialized() override;

	UE_API virtual FZoneGraphTagMask GetAnnotationTags() const override;

	UE_API virtual void TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& AnnotationTagContainer) override;
	UE_API virtual void HandleEvents(const FInstancedStructContainer& Events) override;

	UE_API virtual void PostZoneGraphDataAdded(const AZoneGraphData& ZoneGraphData) override;
	UE_API virtual void PreZoneGraphDataRemoved(const AZoneGraphData& ZoneGraphData) override;

	UE_API void UpdateDangerLanes();
	UE_API void UpdateAnnotationTags(FZoneGraphAnnotationTagContainer& AnnotationTagContainer);

#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual void DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy) override;
	UE_API virtual void DebugDrawCanvas(UCanvas* Canvas, APlayerController*) override;

	FVector LastDebugDrawLocation = FVector::ZeroVector; 
#endif // UE_ENABLE_DEBUG_DRAWING

	UE_API void CalculateEscapeGraph(FZoneGraphDataEscapeGraph& EscapeGraph);

	/** Tag to mark the lanes that should be fled. */
	UPROPERTY(Category="Annotation", EditAnywhere)
	FZoneGraphTag DangerAnnotationTag;

	/** Tag to mark the lanes influenced by an obstacle. */
	UPROPERTY(Category="Annotation", EditAnywhere)
	FZoneGraphTag ObstacleAnnotationTag;
	
	/** Filter specifying which lanes the Annotation is applied to. */
	UPROPERTY(Category="Annotation", EditAnywhere)
	FZoneGraphTagFilter AffectedLaneTags;

	/** Filter specifying which lanes can be used during Disturbance. */
	UPROPERTY(Category="Annotation", EditAnywhere)
	FZoneGraphTagFilter EscapeLaneTags;

	/** Ideal span length for lane subdivision. Each lane will have between 2 and 8 spans. */
	UPROPERTY(Category="Annotation", EditAnywhere)
	float IdealSpanLength = 500.0f;

	/** Array of currently active dangers. */
	TArray<FZoneGraphDisturbanceArea> Dangers;

	/** Array of obstacles. */
	TArray<FZoneGraphObstacleDisturbanceArea> Obstacles;
	
	/** Flag indicating if the event processing changed the dangers. */
	bool bDisturbancesChanged = false;

	/** Disturbance graph for each ZoneGraphData */ 
	TArray<FZoneGraphDataEscapeGraph> EscapeGraphs; 

	/** Combined mask of tags added in previous update (used for clearing previous state) */
	FZoneGraphTagMask PreviouslyAppliedTags = FZoneGraphTagMask::None;

	/** Cached ZoneGraphSubsystem */
	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphSubsystem> ZoneGraphSubsystem = nullptr;
};

/**
 * Test for Disturbance Annotation
 */
UCLASS(MinimalAPI)
class UZoneGraphDisturbanceAnnotationTest : public UZoneGraphAnnotationTest
{
	GENERATED_BODY()

public:
	
#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual FBox CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual void DebugDraw(FDebugRenderSceneProxy* DebugProxy) override;

#endif // UE_ENABLE_DEBUG_DRAWING

	UE_API virtual void Trigger() override;

protected:

	UPROPERTY(EditAnywhere, Category = "Test")
	FVector Offset;

	UPROPERTY(EditAnywhere, Category = "Test")
	float Duration = 10.0f;

	UPROPERTY(EditAnywhere, Category = "Test")
	float DangerRadius = 500.0f;
};

#undef UE_API
