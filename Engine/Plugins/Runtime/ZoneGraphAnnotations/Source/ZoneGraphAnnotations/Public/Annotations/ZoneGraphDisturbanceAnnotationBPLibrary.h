// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphDisturbanceAnnotationBPLibrary.generated.h"

#define UE_API ZONEGRAPHANNOTATIONS_API

/**
 *	Set of utilities for dealing with Disturbance Annotation.
 */
UCLASS(MinimalAPI)
class UZoneGraphDisturbanceAnnotationBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/*
	 * Triggers Danger event at specific location.
	 * @param Instigator (optional) identifies this event coming from specific Instigator, only one danger will persist per instigator.
	 * @param Position Position of the danger.
	 * @param Radius Radius of the danger.
	 * @param Duration Duration of the danger.
	 */
	UFUNCTION(BlueprintCallable, Category = "ZoneGraphAnnotations", meta = (WorldContext = "WorldContextObject"))
	static UE_API void TriggerDanger(UObject* WorldContextObject, const AActor* Instigator, const FVector Position, const float Radius, const float Duration);
	
};

#undef UE_API
