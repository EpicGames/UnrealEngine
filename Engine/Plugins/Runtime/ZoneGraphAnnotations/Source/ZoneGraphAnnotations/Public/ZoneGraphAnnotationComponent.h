// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "DebugRenderSceneProxy.h"
#include "Engine/World.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphAnnotationComponent.generated.h"

#define UE_API ZONEGRAPHANNOTATIONS_API

class UZoneGraphAnnotationSubsystem;
class UZoneGraphAnnotationComponent;
class UCanvas;
class AZoneGraphData;
struct FInstancedStructContainer;
struct FZoneGraphAnnotationTagLookup;
struct FZoneGraphAnnotationTagContainer;

#if UE_ENABLE_DEBUG_DRAWING
class FZoneGraphAnnotationSceneProxy final : public FDebugRenderSceneProxy
{
public:
	UE_API FZoneGraphAnnotationSceneProxy(const UPrimitiveComponent& InComponent, const EDrawType InDrawType = EDrawType::WireMesh);
	
	UE_API virtual SIZE_T GetTypeHash() const override;
	UE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	UE_API virtual uint32 GetMemoryFootprint(void) const override;

private:
	uint32 ViewFlagIndex = 0;
};
#endif


UCLASS(MinimalAPI, Abstract)
class UZoneGraphAnnotationComponent : public UDebugDrawComponent
{
	GENERATED_BODY()

public:
	UE_API UZoneGraphAnnotationComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Called during OnRegister(), or after all subsystems have been initialized. */
	UE_API virtual void PostSubsystemsInitialized();
	
	/** Ticks the Annotation and changes the tags in the container when needed. */
	virtual void TickAnnotation(const float DeltaTime, FZoneGraphAnnotationTagContainer& AnnotationTagContainer) {}

	/** Called when new events are ready to be processed */
	virtual void HandleEvents(const FInstancedStructContainer& Events) {}

	/** @return Tags applied by the Annotation, used to lookup Annotations from tags. */
	virtual FZoneGraphTagMask GetAnnotationTags() const { return FZoneGraphTagMask::None; }

	/** Called when new ZoneGraph data is added. */ 
	virtual void PostZoneGraphDataAdded(const AZoneGraphData& ZoneGraphData) {}

	/** Called when new ZoneGraph data is removed. */ 
	virtual void PreZoneGraphDataRemoved(const AZoneGraphData& ZoneGraphData) {}
	
#if UE_ENABLE_DEBUG_DRAWING
	/** Returns first view point (player controller or debug camera) */
	UE_API void GetFirstViewPoint(FVector& ViewLocation, FRotator& ViewRotation) const;

	/** Returns ZoneGraph max debug draw distance. */
	UE_API float GetMaxDebugDrawDistance() const;

	/** Called when scene proxy is rebuilt. */
	virtual void DebugDraw(FZoneGraphAnnotationSceneProxy* DebugProxy) {}

	/** Called when it's time to draw to canvas. */
	virtual void DebugDrawCanvas(UCanvas* Canvas, APlayerController*) {}
#endif

protected:

	UE_API void OnPostZoneGraphDataAdded(const AZoneGraphData* ZoneGraphData);
	UE_API void OnPreZoneGraphDataRemoved(const AZoneGraphData* ZoneGraphData);
	UE_API void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	UE_API virtual void OnRegister()  override;
	UE_API virtual void OnUnregister()  override;

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	FDelegateHandle OnPostZoneGraphDataAddedHandle;
	FDelegateHandle OnPreZoneGraphDataRemovedHandle;
	FDelegateHandle OnPostWorldInitDelegateHandle;

#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	FDelegateHandle CanvasDebugDrawDelegateHandle;
#endif
	
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bEnableDebugDrawing = false;
};

#undef UE_API
