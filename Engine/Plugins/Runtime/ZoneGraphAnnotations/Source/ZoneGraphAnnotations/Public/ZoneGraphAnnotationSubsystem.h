// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStructContainer.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphAnnotationTypes.h"
#include "Misc/MTAccessDetector.h"
#include "Misc/SpinLock.h"
#include "Subsystems/WorldSubsystem.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "ZoneGraphAnnotationSubsystem.generated.h"

#define UE_API ZONEGRAPHANNOTATIONS_API

class UZoneGraphAnnotationComponent;
class AZoneGraphData;


/**
 * Struct holding combined tags for a specific ZoneGraphData.
 */
struct FZoneGraphDataAnnotationTags
{
	TArray<FZoneGraphTagMask> LaneTags;	// Combined array of tags from all Annotations.
	FZoneGraphDataHandle DataHandle;	// Handle of the data
	bool bInUse = false;				// True, if this entry is in use.
};

/**
 * Annotation tags per zone graph data
 */
struct FZoneGraphAnnotationTagContainer
{
	TArrayView<FZoneGraphTagMask> GetMutableAnnotationTagsForData(const FZoneGraphDataHandle DataHandle)
	{
		check(DataAnnotationTags[DataHandle.Index].DataHandle == DataHandle);
		return DataAnnotationTags[DataHandle.Index].LaneTags;
	}

	TArray<FZoneGraphDataAnnotationTags> DataAnnotationTags;

 	/** Mask combining all static tags used by any of the registered ZoneGraphData. */
	FZoneGraphTagMask CombinedStaticTags;
};


// Struct representing registered ZoneGraph data in the subsystem.
USTRUCT()
struct FRegisteredZoneGraphAnnotation
{
	GENERATED_BODY()

	void Reset()
	{
		AnnotationComponent = nullptr;
		AnnotationTags = FZoneGraphTagMask::None;
	}

	UPROPERTY()
	TObjectPtr<UZoneGraphAnnotationComponent> AnnotationComponent = nullptr;

	FZoneGraphTagMask AnnotationTags = FZoneGraphTagMask::None;	// Combination of all registered Annotation tag masks.
};


/**
* A subsystem managing Zonegraph Annotations.
*/
UCLASS(MinimalAPI)
class UZoneGraphAnnotationSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()
	
public:
	UE_API UZoneGraphAnnotationSubsystem();

	/** Registers Annotation component */
	UE_API void RegisterAnnotationComponent(UZoneGraphAnnotationComponent& Component);
	
	/** Unregisters Annotation component */
	UE_API void UnregisterAnnotationComponent(UZoneGraphAnnotationComponent& Component);

	/** Sends an event to the Annotations. */
	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, FZoneGraphAnnotationEventBase>::IsDerived, void>::Type SendEvent(const T& InRequest)
	{
		FConstStructView Event = FConstStructView::Make(InRequest);

		// Allow multiple threads to append events (as long as they don't do so while the subsystem gets ticked)
		UE::TScopeLock ScopeLock(EventsLock);
		Events[CurrentEventStream].Append(MakeArrayView(&Event, 1));
	}

	/** @return bitmask of Annotation tags at given lane */
	FZoneGraphTagMask GetAnnotationTags(const FZoneGraphLaneHandle LaneHandle) const
	{
		check(AnnotationTagContainer.DataAnnotationTags.IsValidIndex(LaneHandle.DataHandle.Index));
		const FZoneGraphDataAnnotationTags& AnnotationTags = AnnotationTagContainer.DataAnnotationTags[LaneHandle.DataHandle.Index];
		return AnnotationTags.LaneTags[LaneHandle.Index];
	}

	/** @return First Annotation matching a bit in the bitmask */
	UZoneGraphAnnotationComponent* GetFirstAnnotationForTag(const FZoneGraphTag AnnotationTag) const
	{
		return AnnotationTag.IsValid() ? TagToAnnotationLookup[AnnotationTag.Get()] : nullptr;
	}

	/** Signals the subsystem to re-register all tags. */
#if WITH_EDITOR
	UE_API void ReregisterTagsInEditor();
#endif

protected:

	UE_API void PostZoneGraphDataAdded(const AZoneGraphData* ZoneGraphData);
	UE_API void PreZoneGraphDataRemoved(const AZoneGraphData* ZoneGraphData);

	UE_API void AddToAnnotationLookup(UZoneGraphAnnotationComponent& Annotation, const FZoneGraphTagMask AnnotationTags);
	UE_API void RemoveFromAnnotationLookup(UZoneGraphAnnotationComponent& Annotation);
	
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;

	/** Array of registered components. */
	UPROPERTY(Transient)
	TArray<FRegisteredZoneGraphAnnotation> RegisteredComponents;

	/** Stream of events to be processed, double buffered. */
	UPROPERTY(Transient)
	FInstancedStructContainer Events[2];
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(EventsDetector);
	UE::FSpinLock EventsLock;

	/** Lookup table from tag index to Annotation */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UZoneGraphAnnotationComponent>> TagToAnnotationLookup;
	
	/** Combined tags for each ZoneGraphData. Each ZoneGraphData is indexed by it's data handle index, so there can be gaps in the array. */
	FZoneGraphAnnotationTagContainer AnnotationTagContainer;

	FDelegateHandle OnPostZoneGraphDataAddedHandle;
	FDelegateHandle OnPreZoneGraphDataRemovedHandle;

private:
	/**
	 * Index of the current event stream.
	 * It's marked as "private" to ensure the correctness of the assumptions regarding when it can change. 
	 */
	int32 CurrentEventStream = 0;
};

template<>
struct TMassExternalSubsystemTraits<UZoneGraphAnnotationSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false
	};
};

#undef UE_API
