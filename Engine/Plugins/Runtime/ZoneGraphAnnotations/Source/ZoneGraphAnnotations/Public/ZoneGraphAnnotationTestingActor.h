// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Debug/DebugDrawComponent.h"
#include "DebugRenderSceneProxy.h"
#include "ZoneGraphAnnotationTestingActor.generated.h"

#define UE_API ZONEGRAPHANNOTATIONS_API

class UZoneGraphAnnotationTestingComponent;
class FDebugRenderSceneProxy;

/** Base class for ZoneGraph Annotation tests. */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UZoneGraphAnnotationTest : public UObject
{
	GENERATED_BODY()

public:
	virtual void Trigger() {}

#if UE_ENABLE_DEBUG_DRAWING
	virtual FBox CalcBounds(const FTransform& LocalToWorld) const { return FBox(ForceInit); };
	virtual void DebugDraw(FDebugRenderSceneProxy* DebugProxy) {}
	virtual void DebugDrawCanvas(UCanvas* Canvas, APlayerController*) {}
#endif
	
	void SetOwner(UZoneGraphAnnotationTestingComponent* Owner) { OwnerComponent = Owner; OnOwnerSet(); }
	const UZoneGraphAnnotationTestingComponent* GetOwner() const { return OwnerComponent; }

protected:
	virtual void OnOwnerSet() {}

	UPROPERTY()
	TObjectPtr<UZoneGraphAnnotationTestingComponent> OwnerComponent;
};


/** Debug component to test Mass ZoneGraph Annotations. Handles tests and rendering. */
UCLASS(MinimalAPI, ClassGroup = Debug)
class UZoneGraphAnnotationTestingComponent : public UDebugDrawComponent
{
	GENERATED_BODY()
public:
	UE_API UZoneGraphAnnotationTestingComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	UE_API void Trigger();

protected:
	
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;


#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	UE_API virtual void DebugDraw(FDebugRenderSceneProxy* DebugProxy);
	UE_API virtual void DebugDrawCanvas(UCanvas* Canvas, APlayerController*);

	FDelegateHandle CanvasDebugDrawDelegateHandle;
#endif

	UPROPERTY(EditAnywhere, Category = "Test", Instanced)
	TArray<TObjectPtr<UZoneGraphAnnotationTest>> Tests;
};


/** Debug actor to test Mass ZoneGraph Annotations. */
UCLASS(MinimalAPI, HideCategories = (Actor, Input, Collision, Rendering, Replication, Partition, HLOD, Cooking))
class AZoneGraphAnnotationTestingActor : public AActor
{
	GENERATED_BODY()
public:
	UE_API AZoneGraphAnnotationTestingActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/**
	 * Simple trigger function to trigger something on the tests.
	 * Ideally this would be part of each test, but it does not work there.
	 */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Test")
	UE_API void Trigger();

protected:
#if WITH_EDITOR
	UE_API virtual void PostEditMove(bool bFinished) override;
#endif

	UPROPERTY(Category = Default, VisibleAnywhere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UZoneGraphAnnotationTestingComponent> TestingComp;
};

#undef UE_API
