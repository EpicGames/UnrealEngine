// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ZoneGraphAnnotationTypes.generated.h"

#define UE_API ZONEGRAPHANNOTATIONS_API

ZONEGRAPHANNOTATIONS_API DECLARE_LOG_CATEGORY_EXTERN(LogZoneGraphAnnotations, Warning, All);

/** Base structs for all events for UZoneGraphAnnotationSubsystem. */
USTRUCT()
struct FZoneGraphAnnotationEventBase
{
	GENERATED_BODY()
};

class FMassLaneObstacleID
{
public:
	FMassLaneObstacleID() {}
	
	static UE_API const FMassLaneObstacleID InvalidID;
	static FMassLaneObstacleID GetNextUniqueID()
	{
		check(NextUniqueID < MAX_uint64); // Ran out of FMassLaneObstacleID.
		return FMassLaneObstacleID(NextUniqueID++);
	}
	
	bool operator==(const FMassLaneObstacleID& Other) const
	{
		return Value == Other.Value;
	}

	uint64 GetValue() const { return Value; }

	bool IsValid() const { return Value != MAX_uint64; }

private:
	static UE_API uint64 NextUniqueID;
	
	FMassLaneObstacleID(uint64 ID) : Value(ID) {}
	uint64 Value = MAX_uint64;
};

inline uint32 GetTypeHash(const FMassLaneObstacleID& Obs)
{
	return ::GetTypeHash(Obs.GetValue());
}

#undef UE_API
